{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "PCG Primitives",
	"Description": "PCG Primitives and Examples Library for World Building",
	"Category": "Other",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Plugins": [
		{
			"Name": "PCG",
			"Enabled": true
		},
		{
			"Name": "PCGGeometryScriptInterop",
			"Enabled": true
		},
		{
			"Name": "PCGBiomeCore",
			"Enabled": true
		},
		{
			"Name": "PCGBiomeSample",
			"Enabled": true
		},
		{
			"Name": "PCGExternalDataInterop",
			"Enabled": true
		}
	]
}